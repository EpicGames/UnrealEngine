// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/PlatformMisc.h"
#include "HAL/IConsoleManager.h"
#include "Iris/IrisConfig.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/DelayedAutoRegister.h"
#include "Modules/ModuleManager.h"
#include "Net/Core/Trace/Private/NetTraceInternal.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "SlateGlobals.h"
#include "Styling/SlateWidgetStyleContainerBase.h"
#include "TestCommon/Initialization.h"
#include "AutoRTFM/AutoRTFMUE.h"

#include <catch2/catch_test_macros.hpp>

namespace
{
	FDelayedAutoRegisterHelper DelayedAutoRegister_PreObjectSystemReady(EDelayedRegisterRunPhase::PreObjectSystemReady, []
	{
		// Set up Iris config here in code so we can avoid having to load ini files from disk.
		const TArray<FString> StructsSupportingNetSerializerList =
		{
			TEXT("(StructName=RootMotionSource)"),
			TEXT("(StructName=RootMotionSource_ConstantForce)"),
			TEXT("(StructName=RootMotionSource_RadialForce)"),
			TEXT("(StructName=RootMotionSource_MoveToForce)"),
			TEXT("(StructName=RootMotionSource_MoveToDynamicForce)"),
			TEXT("(StructName=RootMotionSource_JumpForce)")
		};

		GConfig->SetArray(TEXT("/Script/IrisCore.ReplicationStateDescriptorConfig"), TEXT("SupportsStructNetSerializerList"), StructsSupportingNetSerializerList, GEngineIni);
	});
}

GROUP_BEFORE_GLOBAL(Catch::DefaultGroup)
{
	// Add command-line options always needed for the tests
	FCommandLine::Append(TEXT(" -nullrhi -unattended"));

	// Initialize trace
	FString Parameter;
	if (FParse::Value(FCommandLine::Get(), TEXT("-trace="), Parameter, false))
	{
		FTraceAuxiliary::Initialize(FCommandLine::Get());
		FTraceAuxiliary::TryAutoConnect();
	}

#if UE_NET_TRACE_ENABLED
	uint32 NetTraceVerbosity;
	if(FParse::Value(FCommandLine::Get(), TEXT("-nettrace="), NetTraceVerbosity))
	{
		FNetTrace::SetTraceVerbosity(NetTraceVerbosity);
	}
#endif

	UE::Net::SetUseIrisReplication(true);

	if (IConsoleVariable* PushModelCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("Net.IsPushModelEnabled")))
	{
		PushModelCVar->Set(true);
	}

	{
		// Find the engine directory before InitAll() enables the platform file stub. Prevents warnings.
		FPlatformMisc::EngineDir();

		// Silence some errors & warnings during initialization unrelated to the replication system
		LOG_SCOPE_VERBOSITY_OVERRIDE(LogSlate, ELogVerbosity::Error);
		LOG_SCOPE_VERBOSITY_OVERRIDE(LogSlateStyle, ELogVerbosity::Error);
		LOG_SCOPE_VERBOSITY_OVERRIDE(LogUObjectGlobals, ELogVerbosity::Fatal);
		LOG_SCOPE_VERBOSITY_OVERRIDE(LogStreaming, ELogVerbosity::Error);
		InitAll(true, true);
		AutoRTFM::InitializeForUE(nullptr);
	}

	for (const FName ModuleToLoad : {"IrisCore", "ReplicationSystemTestPlugin"})
	{
		FModuleManager::Get().LoadModule(ModuleToLoad);
	}
}

GROUP_AFTER_GLOBAL(Catch::DefaultGroup)
{
	CleanupAll();
}
