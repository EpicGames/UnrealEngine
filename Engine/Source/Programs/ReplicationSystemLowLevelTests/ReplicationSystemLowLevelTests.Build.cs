// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ReplicationSystemLowLevelTests : TestModuleRules
{
	static ReplicationSystemLowLevelTests()
	{
		if (InTestMode)
		{
			TestMetadata = new Metadata();
			TestMetadata.TestName = "ReplicationSystem";
			TestMetadata.TestShortName = "Replication System";
			TestMetadata.ReportType = "testdata";
			TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.Linux);
			TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.LinuxArm64);
			TestMetadata.PlatformRunContainerized.Add(UnrealTargetPlatform.LinuxArm64, true);
		}
	}

	public ReplicationSystemLowLevelTests(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"IrisCore",
				"ReplicationSystemTestPlugin",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[]
			{
				"SlateCore",
			}
		);
	}
}
