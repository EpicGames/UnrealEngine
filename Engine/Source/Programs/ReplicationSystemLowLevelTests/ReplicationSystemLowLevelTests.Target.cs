// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.All)]
public class ReplicationSystemLowLevelTestsTarget : TestTargetRules
{
	public ReplicationSystemLowLevelTestsTarget(TargetInfo Target) : base(Target)
	{
		bTestsRequireEngine = true;
		bTestsRequireApplicationCore = true;
		bTestsRequireCoreUObject = true;
		bUsePlatformFileStub = true;
		bMockEngineDefaults = true;

		bUsesSlate = false;

		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;

		// Network config
		bWithPushModel = true;

		bEnableTrace = true;

		if (!bGenerateProjectFiles && (Target.Platform.IsInGroup(UnrealPlatformGroup.Linux) || Target.Platform == UnrealTargetPlatform.Win64))
		{
			//bUseAutoRTFMCompiler = true;
		}

		GlobalDefinitions.Add("UE_NET_TEST_FAKE_REP_TAGS=1");
		// Load time profiling brings object construction to a crawl.
		GlobalDefinitions.Add("LOADTIMEPROFILERTRACE_ENABLED=0");
		GlobalDefinitions.Add("UE_WITH_REMOTE_OBJECT_HANDLE=0");
	}
}
